{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Message Bus Insights",
	"Description": "Allows debugging of Message Bus via Unreal Insights",
	"Category": "Insights",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Plugins": [
		{
			"Name": "UdpMessaging",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "UdpMessagingInsights",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default",
			"ProgramAllowList": [
				"UnrealInsights"
			]
		}
	]
}
