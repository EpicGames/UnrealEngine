// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "UdpMessagingTraceModule.h"
#include "UdpMessagingTimingViewExtender.h"


namespace UE::MessagingInsights
{


class FUdpMessagingInsightsModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
		IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	}

	virtual void ShutdownModule() override
	{
		IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
		IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	}
	//~ End IModuleInterface interface

private:
	FUdpMessagingTraceModule TraceModule;
	FUdpMessagingTimingViewExtender TimingViewExtender;
};


} // namespace UE::MessagingInsights


IMPLEMENT_MODULE(UE::MessagingInsights::FUdpMessagingInsightsModule, UdpMessagingInsights);
