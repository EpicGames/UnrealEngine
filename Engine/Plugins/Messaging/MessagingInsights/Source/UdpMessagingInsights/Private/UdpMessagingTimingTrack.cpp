// Copyright Epic Games, Inc. All Rights Reserved.

#include "UdpMessagingTimingTrack.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "Math/Color.h"
#include "Math/UnitConversion.h"
#include "Trace/UdpMessagingTrace.h"
#include "UdpMessagingTraceProvider.h"
#include "UdpMessagingTimingViewSession.h"


#define LOCTEXT_NAMESPACE "FUdpMessagingTimingTrack"


namespace UE::MessagingInsights
{


class FMessageTimingEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FMessageTimingEvent, FTimingEvent)

public:
	FMessageTimingEvent(
		const TSharedRef<const FBaseTimingTrack>& InTrack,
		double InStartTime,
		double InEndTime,
		uint32 InDepth,
		const FMessageLifecycleEvent& InEvent
	)
		: FTimingEvent(InTrack, InStartTime, InEndTime, InDepth)
		, MessageEvent(InEvent)
	{
	}

	FMessageLifecycleEvent MessageEvent;
	const TCHAR* MessageTypePackage = nullptr;
	const TCHAR* MessageTypeAsset = nullptr;
	TOptional<FMessageSummary> MessageSummary;
	TOptional<FNodeDiscoveredEvent> SenderNode;
	TOptional<FNodeDiscoveredEvent> RecipientNode;
};


INSIGHTS_IMPLEMENT_RTTI(FMessageTimingEvent)
INSIGHTS_IMPLEMENT_RTTI(FUdpMessagingTimingTrack)


FUdpMessagingTimingTrack::FUdpMessagingTimingTrack(
	const FUdpMessagingTimingViewSession& InSession,
	const uint16 InNodeShortId,
	const EMessageDirection InDirection
)
	: Super()
	, SharedData(InSession)
	, NodeShortId(InNodeShortId)
	, Direction(InDirection)
{
}


namespace Private
{
uint32 GetSpanColor(const FMessageLifecycleEvent& InEvent)
{
	using UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent;
	FLinearColor HSV(float(InEvent.MessageId) * UE_GOLDEN_RATIO * 360.f, 1.0f, 0.75f);

	bool bIsInner = false;
	int32 EventNum = 0;
	if (InEvent.Details.IsType<FMessageLifecycleEvent::FInnerEvent>())
	{
		const FMessageLifecycleEvent::FInnerEvent& InnerDetails =
			InEvent.Details.Get<FMessageLifecycleEvent::FInnerEvent>();

		bIsInner = true;
		EventNum = static_cast<int32>(InnerDetails.EventType);
	}
	else
	{
		const FMessageLifecycleEvent::FOuterEvent& OuterDetails =
			InEvent.Details.Get<FMessageLifecycleEvent::FOuterEvent>();

		EventNum = static_cast<int32>(OuterDetails.TerminalEventType);
	}

	if (EventNum >= static_cast<int32>(EUdpMessageLifecycleEvent::Discarded))
	{
		// Exceptional event; Insights idiom is to color it black.
		HSV.B = 0.f;
	}
	else if (bIsInner)
	{
		if (EventNum >= static_cast<int32>(EUdpMessageLifecycleEvent::FirstSegmentEnqueueInbound))
		{
			// Receive event; make 0-relative
			EventNum -= static_cast<int32>(EUdpMessageLifecycleEvent::FirstSegmentEnqueueInbound);
		}

		const float HueDegPerEvent = 30.f;
		HSV.R += HueDegPerEvent * EventNum;
		HSV.G *= 0.5f;
	}

	return HSV.HSVToLinearRGB().ToFColor(false).ToPackedARGB();
}
}


void FUdpMessagingTimingTrack::BuildDrawState(
	ITimingEventsTrackDrawStateBuilder& Builder,
	const ITimingTrackUpdateContext& Context
)
{
	const FUdpMessagingProvider* Provider =
		SharedData.GetAnalysisSession().ReadProvider<FUdpMessagingProvider>(FUdpMessagingProvider::ProviderName);
	if (!Provider)
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	const FUdpMessagingProvider::FMessageTimeline* Timeline = Provider->GetMessageTimeline(NodeShortId, Direction);
	if (!ensure(Timeline))
	{
		return;
	}

	auto BuildDrawEvents =
		[this, &Builder, &Viewport, Provider]
		(double InStartTime, double InEndTime, uint32 InDepth, const FMessageLifecycleEvent& MessageEvent)
		{
			TStringBuilder<128> NameBuilder;
			if (MessageEvent.Details.IsType<FMessageLifecycleEvent::FInnerEvent>())
			{
				if (!bShowLifecycleEvents)
				{
					return TraceServices::EEventEnumerate::Continue;
				}

				const FMessageLifecycleEvent::FInnerEvent& InnerDetails =
					MessageEvent.Details.Get<FMessageLifecycleEvent::FInnerEvent>();
				NameBuilder.Appendf(TEXT("%s"), UE::UdpMessaging::Trace::LexToString(InnerDetails.EventType));
			}
			else
			{
				TPair<const TCHAR*, const TCHAR*> TypeInfo =
					Provider->GetMessageTypeInfo(MessageEvent.SenderShortId, MessageEvent.RecipientShortId, MessageEvent.MessageId);

				if (TypeInfo.Value)
				{
					NameBuilder.Appendf(TEXT("%s"), TypeInfo.Value);
				}
				else
				{
					NameBuilder.Appendf(TEXT("Message %u"), MessageEvent.MessageId);
				}
			}

			const uint32 EffectiveLaneDepth = bShowLifecycleEvents ? MessageEvent.LaneDepth : MessageEvent.LaneDepth / 2;
			Builder.AddEvent(InStartTime, InEndTime, EffectiveLaneDepth, *NameBuilder, 0, Private::GetSpanColor(MessageEvent));

			return TraceServices::EEventEnumerate::Continue;
		};

	Timeline->EnumerateEvents(Viewport.GetStartTime(), Viewport.GetEndTime(), BuildDrawEvents);
	Provider->EnumerateOutstandingMessages(NodeShortId, Direction, Viewport.GetStartTime(), Viewport.GetEndTime(), BuildDrawEvents);
}

const TSharedPtr<const ITimingEvent> FUdpMessagingTimingTrack::GetEvent(double InTime, double SecondsPerPixel, int32 InDepth) const
{
	const FUdpMessagingProvider* Provider =
		SharedData.GetAnalysisSession().ReadProvider<FUdpMessagingProvider>(FUdpMessagingProvider::ProviderName);
	if (!Provider)
	{
		return nullptr;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());

	double Delta = 2.0 * SecondsPerPixel;
	FMessageLifecycleEvent BestMatchEvent;
	double BestGraphStartTime = 0.0;
	double BestGraphEndTime = 0.0;
	uint32 BestDepth = 0;
	auto SetBestMatchEvent =
		[&BestMatchEvent, &BestGraphStartTime, &BestGraphEndTime, &BestDepth]
		(double GraphStartTime, double GraphEndTime, uint32 Depth, const FMessageLifecycleEvent& MessageEvent)
		{
			BestMatchEvent = MessageEvent;
			BestGraphStartTime = GraphStartTime;
			BestGraphEndTime = GraphEndTime;
			BestDepth = Depth;
		};

	const FUdpMessagingProvider::FMessageTimeline* Timeline = Provider->GetMessageTimeline(NodeShortId, Direction);
	if (!ensure(Timeline))
	{
		return nullptr;
	}

	auto TryFindBetterMatchEvent =
		[bShowLifecycleEvents = bShowLifecycleEvents, InTime, &Delta, &InDepth, &SetBestMatchEvent]
		(double GraphStartTime, double GraphEndTime, uint32, const FMessageLifecycleEvent& MessageEvent)
		{
			const uint32 EffectiveInDepth = bShowLifecycleEvents ? InDepth : InDepth * 2;
			if (GraphStartTime <= InTime && GraphEndTime >= InTime && EffectiveInDepth == MessageEvent.LaneDepth)
			{
				Delta = 0.0f;
				SetBestMatchEvent(GraphStartTime, GraphEndTime, InDepth, MessageEvent);
				return TraceServices::EEventEnumerate::Stop;
			}

			double DeltaLeft = InTime - GraphEndTime;
			if (DeltaLeft >= 0 && DeltaLeft < Delta && EffectiveInDepth == MessageEvent.LaneDepth)
			{
				Delta = DeltaLeft;
				SetBestMatchEvent(GraphStartTime, GraphEndTime, InDepth, MessageEvent);
			}

			double DeltaRight = GraphStartTime - InTime;
			if (DeltaRight >= 0 && DeltaRight < Delta && EffectiveInDepth == MessageEvent.LaneDepth)
			{
				Delta = DeltaRight;
				SetBestMatchEvent(GraphStartTime, GraphEndTime, InDepth, MessageEvent);
			}

			return TraceServices::EEventEnumerate::Continue;
		};

	Timeline->EnumerateEvents(InTime - Delta, InTime + Delta, TryFindBetterMatchEvent);
	Provider->EnumerateOutstandingMessages(NodeShortId, Direction, InTime - Delta, InTime + Delta, TryFindBetterMatchEvent);

	if (Delta < 2.0 * SecondsPerPixel)
	{
		TSharedRef<FMessageTimingEvent> Event = MakeShared<FMessageTimingEvent>(SharedThis(this), BestGraphStartTime, BestGraphEndTime, BestDepth, BestMatchEvent);

		TPair<const TCHAR*, const TCHAR*> TypeInfo =
			Provider->GetMessageTypeInfo(BestMatchEvent.SenderShortId, BestMatchEvent.RecipientShortId, BestMatchEvent.MessageId);
		Event->MessageTypePackage = TypeInfo.Get<0>();
		Event->MessageTypeAsset = TypeInfo.Get<1>();

		if (const FMessageSummary* Summary = Provider->GetMessageSummary(BestMatchEvent.SenderShortId, BestMatchEvent.RecipientShortId, BestMatchEvent.MessageId))
		{
			Event->MessageSummary = *Summary;
		}
		if (const FNodeDiscoveredEvent* SenderNode = Provider->GetNodeByShortId(BestMatchEvent.SenderShortId))
		{
			Event->SenderNode = *SenderNode;
		}
		if (const FNodeDiscoveredEvent* RecipientNode = Provider->GetNodeByShortId(BestMatchEvent.RecipientShortId))
		{
			Event->RecipientNode = *RecipientNode;
		}
		return Event;
	}

	return nullptr;
}

void FUdpMessagingTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	InOutTooltip.ResetContent();

	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FMessageTimingEvent>())
	{
		return;
	}

	const FMessageTimingEvent& TimingEvent = InTooltipEvent.As<FMessageTimingEvent>();

	InOutTooltip.AddTitle(FString::Printf(TEXT("Message ID: %u"), TimingEvent.MessageEvent.MessageId));

	EnumerateTimingEventFields(TimingEvent,
		[&InOutTooltip](FStringView FieldName, FStringView FieldValue)
		{
			InOutTooltip.AddNameValueTextLine(FieldName, FieldValue);
		});

	InOutTooltip.UpdateLayout();
}

void FUdpMessagingTimingTrack::OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const
{
	if (!InSelectedEvent.CheckTrack(this) || !InSelectedEvent.Is<FMessageTimingEvent>())
	{
		return;
	}

	const FMessageTimingEvent& TimingEvent = InSelectedEvent.As<FMessageTimingEvent>();

	TStringBuilder<2048> Builder;
	Builder.Appendf(TEXT("Message ID: %u\n"), TimingEvent.MessageEvent.MessageId);

	EnumerateTimingEventFields(TimingEvent,
		[&Builder](FStringView FieldName, FStringView FieldValue)
		{
			Builder.Appendf(TEXT("%.*s %.*s\n"),
				FieldName.Len(), FieldName.GetData(),
				FieldValue.Len(), FieldValue.GetData());
		});

	FPlatformApplicationMisc::ClipboardCopy(*Builder);
}

void FUdpMessagingTimingTrack::EnumerateTimingEventFields(const FMessageTimingEvent& InTimingEvent, TFunctionRef<void(FStringView, FStringView)> InCallback) const
{
	TStringBuilder<256> Builder;

	InCallback(TEXTVIEW("Start Time:"), UE::Insights::FormatTimeAuto(InTimingEvent.GetStartTime(), 6));
	InCallback(TEXTVIEW("End Time:"), UE::Insights::FormatTimeAuto(InTimingEvent.GetEndTime(), 6));
	InCallback(TEXTVIEW("Duration:"), UE::Insights::FormatTimeAuto(InTimingEvent.GetDuration()));
	if (InTimingEvent.MessageEvent.Details.IsType<FMessageLifecycleEvent::FInnerEvent>())
	{
		const FMessageLifecycleEvent::FInnerEvent& InnerDetails =
			InTimingEvent.MessageEvent.Details.Get<FMessageLifecycleEvent::FInnerEvent>();
		InCallback(TEXTVIEW("Event:"), UE::UdpMessaging::Trace::LexToString(InnerDetails.EventType));
	}
	else
	{
		InCallback(TEXT("Type Package:"), InTimingEvent.MessageTypePackage);
		InCallback(TEXT("Type Asset:"), InTimingEvent.MessageTypeAsset);
		if (InTimingEvent.MessageSummary)
		{
			Builder.Reset();
			const uint64 SizeBytes = InTimingEvent.MessageSummary->Size;
			const FNumericUnit<uint64> SizeUnit = FUnitConversion::QuantizeUnitsToBestFit(SizeBytes, EUnit::Bytes);
			Builder.Appendf(TEXT("%llu %s"), SizeUnit.Value, FUnitConversion::GetUnitDisplayString(SizeUnit.Units));
			Builder.Appendf(TEXT(" (%s bytes)"), *FText::AsNumber(SizeBytes).ToString());
			InCallback(TEXTVIEW("Size:"), Builder.ToView());
			
			Builder.Reset();
			Builder.Appendf(TEXT("%u"), InTimingEvent.MessageSummary->NumSegments);
			InCallback(TEXTVIEW("# Segments:"), Builder.ToView());
		}

		if (InTimingEvent.SenderNode)
		{
			InCallback(TEXTVIEW("Sender:"), InTimingEvent.SenderNode->Endpoint.ToString());
		}

		if (InTimingEvent.RecipientNode)
		{
			InCallback(TEXTVIEW("Recipient:"), InTimingEvent.RecipientNode->Endpoint.ToString());
		}
	}
}

void FUdpMessagingTimingTrack::BuildContextMenu(FMenuBuilder& InOutMenuBuilder)
{
	InOutMenuBuilder.BeginSection("UdpMessaging", LOCTEXT("ContextMenu_SectionHeading", "UdpMessaging"));

	InOutMenuBuilder.AddMenuEntry(
		LOCTEXT("ContextMenu_LifecycleEvents_Label", "Lifecycle Events"), 
		LOCTEXT("ContextMenu_LifecycleEvents_Tooltip", "Show/Hide nested individual lifecycle events for messages"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSPLambda(this, [this]() { bShowLifecycleEvents = !bShowLifecycleEvents;}),
			FCanExecuteAction(),
			FIsActionChecked::CreateSPLambda(this, [this]() { return bShowLifecycleEvents; })
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);

	InOutMenuBuilder.EndSection();
}


} // namespace UE::MessagingInsights


#undef LOCTEXT_NAMESPACE
