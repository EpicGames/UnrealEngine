// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/TimingEventsTrack.h"

class ITimingTrackUpdateContext;


namespace UE::MessagingInsights
{


enum class EMessageDirection : uint8;
class FMessageTimingEvent;
class FUdpMessagingTimingViewSession;


/** Timing track for UDP Messaging. */
class FUdpMessagingTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FUdpMessagingTimingTrack, FTimingEventsTrack)
	using Super = FTimingEventsTrack;

public:
	FUdpMessagingTimingTrack(
		const FUdpMessagingTimingViewSession& InSharedData,
		const uint16 InNodeShortId,
		const EMessageDirection InDirection
	);

	//~ Begin FTimingEventsTrack interface
	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	virtual void OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual const TSharedPtr<const ITimingEvent> GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const override;
	//~ End FTimingEventsTrack interface

private:
	void EnumerateTimingEventFields(const FMessageTimingEvent& InSelectedEvent, TFunctionRef<void(FStringView, FStringView)> InCallback) const;

private:
	const FUdpMessagingTimingViewSession& SharedData;
	const uint16 NodeShortId;
	const EMessageDirection Direction;

	bool bShowLifecycleEvents = true;
};


} //namespace UE::MessagingInsights
