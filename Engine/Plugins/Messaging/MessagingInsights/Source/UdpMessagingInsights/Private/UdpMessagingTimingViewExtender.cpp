// Copyright Epic Games, Inc. All Rights Reserved.

#include "UdpMessagingTimingViewExtender.h"
#include "UdpMessagingTimingViewSession.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"


namespace UE::MessagingInsights
{


void FUdpMessagingTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if (!PerSessionData)
	{
		PerSessionData = &PerSessionDataMap.Add(&InSession);
		PerSessionData->SharedData = MakePimpl<FUdpMessagingTimingViewSession>();
	}

	PerSessionData->SharedData->OnBeginSession(InSession);
}

void FUdpMessagingTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	if (FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession))
	{
		PerSessionData->SharedData->OnEndSession(InSession);
	}

	PerSessionDataMap.Remove(&InSession);
}

void FUdpMessagingTimingViewExtender::Tick(const UE::Insights::Timing::FTimingViewExtenderTickParams& InParams)
{
	if (InParams.Session.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	if (FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InParams.Session))
	{
		PerSessionData->SharedData->Tick(InParams);
	}
}

void FUdpMessagingTimingViewExtender::ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	if (FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession))
	{
		PerSessionData->SharedData->ExtendFilterMenu(InMenuBuilder);
	}
}


} //namespace UE::MessagingInsights
