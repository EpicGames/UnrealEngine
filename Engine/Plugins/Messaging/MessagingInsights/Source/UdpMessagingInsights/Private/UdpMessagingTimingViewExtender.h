// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ITimingViewExtender.h"
#include "Templates/PimplPtr.h"


namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::Insights::Timing { struct FTimingViewExtenderTickParams; }
class FMenuBuilder;


namespace UE::MessagingInsights
{

class FUdpMessagingTimingViewSession;


class FUdpMessagingTimingViewExtender : public UE::Insights::Timing::ITimingViewExtender
{
public:
	//~ Begin UE::Insights::Timing::ITimingViewExtender interface
	virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void Tick(const UE::Insights::Timing::FTimingViewExtenderTickParams& InParams) override;
	virtual void ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;
	//~ End UE::Insights::Timing::ITimingViewExtender interface

private:
	struct FPerSessionData
	{
		TPimplPtr<FUdpMessagingTimingViewSession> SharedData;
	};

	// The data we host per-session
	TMap<UE::Insights::Timing::ITimingViewSession*, FPerSessionData> PerSessionDataMap;
};


} //namespace UE::MessagingInsights
