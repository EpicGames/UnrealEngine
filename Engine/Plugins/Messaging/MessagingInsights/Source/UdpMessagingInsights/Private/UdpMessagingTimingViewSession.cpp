// Copyright Epic Games, Inc. All Rights Reserved.

#include "UdpMessagingTimingViewSession.h"

#include "Insights/ITimingViewExtender.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"

#include "UdpMessagingTimingTrack.h"
#include "UdpMessagingTraceProvider.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"


#define LOCTEXT_NAMESPACE "UdpMessagingTimingViewSession"


namespace UE::MessagingInsights
{


FUdpMessagingTimingViewSession::FUdpMessagingTimingViewSession()
{
}

void FUdpMessagingTimingViewSession::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	if (InTimingViewSession.GetName() == FInsightsManagerTabs::TimingProfilerTabId)
	{
		TimingViewSession = &InTimingViewSession;
	}
}

void FUdpMessagingTimingViewSession::OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	if (&InTimingViewSession != TimingViewSession)
	{
		return;
	}

	LocalNodeTracks.Empty();

	TimingViewSession = nullptr;
}

void FUdpMessagingTimingViewSession::Tick(const UE::Insights::Timing::FTimingViewExtenderTickParams& InParams)
{
	if (&InParams.Session != TimingViewSession)
	{
		return;
	}

	if (!InParams.AnalysisSession)
	{
		return;
	}

	AnalysisSession = InParams.AnalysisSession;

	if (const FUdpMessagingProvider* Provider =
		AnalysisSession->ReadProvider<FUdpMessagingProvider>(FUdpMessagingProvider::ProviderName))
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(GetAnalysisSession());

		// Add timing tracks
		for (uint32 LocalNodeIdx = 0; LocalNodeIdx < Provider->GetNumLocalNodes(); ++LocalNodeIdx)
		{
			const uint16 LocalNodeShortId = Provider->GetLocalNodeShortId(LocalNodeIdx);
			const FUdpMessagingProvider::FMessageTimeline* SendTimeline =
				Provider->GetMessageTimeline(LocalNodeShortId, EMessageDirection::Send);
			const FUdpMessagingProvider::FMessageTimeline* ReceiveTimeline =
				Provider->GetMessageTimeline(LocalNodeShortId, EMessageDirection::Receive);
			if (!ensure(SendTimeline && ReceiveTimeline))
			{
				continue;
			}

			if (SendTimeline->GetEventCount() == 0 && ReceiveTimeline->GetEventCount() == 0)
			{
				continue;
			}

			if (!LocalNodeTracks.Contains(LocalNodeShortId))
			{
				TSharedRef<FUdpMessagingTimingTrack> SendTrack = MakeShared<FUdpMessagingTimingTrack>(*this, LocalNodeShortId, EMessageDirection::Send);
				TSharedRef<FUdpMessagingTimingTrack> ReceiveTrack = MakeShared<FUdpMessagingTimingTrack>(*this, LocalNodeShortId, EMessageDirection::Receive);
				LocalNodeTracks.Emplace(LocalNodeShortId, { .SendTrack = SendTrack, .ReceiveTrack = ReceiveTrack });

				if (const FNodeDiscoveredEvent* NodeInfo = Provider->GetNodeByShortId(LocalNodeShortId))
				{
					SendTrack->SetName(FString::Printf(TEXT("UdpMessaging - %s - Sent"), *NodeInfo->Endpoint.ToString()));
					ReceiveTrack->SetName(FString::Printf(TEXT("UdpMessaging - %s - Received"), *NodeInfo->Endpoint.ToString()));
				}

				SendTrack->SetVisibilityFlag(bMessageTracksEnabled);
				ReceiveTrack->SetVisibilityFlag(bMessageTracksEnabled);
				TimingViewSession->AddScrollableTrack(SendTrack);
				TimingViewSession->AddScrollableTrack(ReceiveTrack);
				TimingViewSession->InvalidateScrollableTracksOrder();
			}
		}
	}
}

void FUdpMessagingTimingViewSession::ExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("UdpMessagingTracks", LOCTEXT("UdpMessagingHeader", "UdpMessaging"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("MessageTrackToggleLabel", "Messages"),
			LOCTEXT("MessageTrackToggleLabel_Tooltip", "Show/hide the Messages tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FUdpMessagingTimingViewSession::ToggleMessageTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return bMessageTracksEnabled; })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();
}


void FUdpMessagingTimingViewSession::ToggleMessageTracks()
{
	bMessageTracksEnabled = !bMessageTracksEnabled;

	for (TPair<uint16, FDuplexTracks>& Node : LocalNodeTracks)
	{
		Node.Value.SendTrack->SetVisibilityFlag(bMessageTracksEnabled);
		Node.Value.ReceiveTrack->SetVisibilityFlag(bMessageTracksEnabled);
	}
}


} //namespace UE::MessagingInsights


#undef LOCTEXT_NAMESPACE
