// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { enum class ETimeChangedFlags : int32; }
namespace UE::Insights::Timing { struct FTimingViewExtenderTickParams; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::MessagingInsights { class FUdpMessagingTimingTrack; }
class FMenuBuilder;


namespace UE::MessagingInsights
{


class FUdpMessagingTimingViewSession
{
public:
	FUdpMessagingTimingViewSession();

	void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void Tick(const UE::Insights::Timing::FTimingViewExtenderTickParams& InParams);
	void ExtendFilterMenu(FMenuBuilder& InMenuBuilder);

	/** Get the last cached analysis session */
	const TraceServices::IAnalysisSession& GetAnalysisSession() const
	{
		check(AnalysisSession);
		return *AnalysisSession;
	}

	/** Show/hide the message tracks. */
	void ToggleMessageTracks();

	/** The timing view for the session. */
	UE::Insights::Timing::ITimingViewSession* GetTimingView() const
	{
		return TimingViewSession;
	}

private:
	// Cached analysis session, set in Tick()
	const TraceServices::IAnalysisSession* AnalysisSession = nullptr;

	// Cached timing view session, set in OnBeginSession/OnEndSession
	UE::Insights::Timing::ITimingViewSession* TimingViewSession = nullptr;

	struct FDuplexTracks
	{
		TSharedRef<FUdpMessagingTimingTrack> SendTrack;
		TSharedRef<FUdpMessagingTimingTrack> ReceiveTrack;
	};

	TMap<uint16, FDuplexTracks> LocalNodeTracks;

	/** Whether message tracks are enabled. */
	bool bMessageTracksEnabled = true;
};


} //namespace UE::MessagingInsights
