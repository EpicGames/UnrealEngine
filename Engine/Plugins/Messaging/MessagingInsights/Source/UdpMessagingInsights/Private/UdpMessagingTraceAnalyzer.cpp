// Copyright Epic Games, Inc. All Rights Reserved.

#include "UdpMessagingTraceAnalyzer.h"
#include "HAL/LowLevelMemTracker.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/Strings.h"
#include "UdpMessagingTraceProvider.h"


namespace UE::MessagingInsights
{


FUdpMessagingAnalyzer::FUdpMessagingAnalyzer(TraceServices::IAnalysisSession& InSession, FUdpMessagingProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{

}


void FUdpMessagingAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_DiscoveredNode, "UdpMessaging", "DiscoveredNode");
	Builder.RouteEvent(RouteId_MessageSummary, "UdpMessaging", "MessageSummary");
	Builder.RouteEvent(RouteId_MessageTypeInfo, "UdpMessaging", "MessageTypeInfo");
	Builder.RouteEvent(RouteId_MessageLifecycle, "UdpMessaging", "MessageLifecycle");
}


void FUdpMessagingAnalyzer::OnAnalysisEnd()
{
}


bool FUdpMessagingAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FUdpMessagingAnalyzer"));

	TraceServices::FAnalysisSessionEditScope _(Session);

	const UE::Trace::IAnalyzer::FEventData& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_DiscoveredNode:
		{
			FNodeDiscoveredEvent Event = FNodeDiscoveredEvent::FromEventData(EventData);
			Provider.AddNodeDiscoveredEvent(Context.EventTime.AsSeconds(Event.Cycle), MoveTemp(Event));
			break;
		}
		case RouteId_MessageSummary:
		{
			FMessageSummary Summary = FMessageSummary::FromEventData(EventData);
			Provider.AddMessageSummary(Summary.SenderShortId, Summary.RecipientShortId, Summary.MessageId, MoveTemp(Summary));
			break;
		}
		case RouteId_MessageTypeInfo:
		{
			const TraceServices::IDefinitionProvider* DefinitionProvider = TraceServices::ReadDefinitionProvider(Session);
			if (DefinitionProvider)
			{
				const uint16 SenderShortId = EventData.GetValue<uint16>("SenderShortId");
				const uint16 RecipientShortId = EventData.GetValue<uint16>("RecipientShortId");
				const uint32 MessageId = EventData.GetValue<uint32>("MessageId");
				const UE::Trace::FEventRef32 PackageNameRef = EventData.GetReferenceValue<uint32>("Package");
				const UE::Trace::FEventRef32 AssetNameRef = EventData.GetReferenceValue<uint32>("Asset");
				const TraceServices::FStringDefinition* PackageName = DefinitionProvider->Get<TraceServices::FStringDefinition>(PackageNameRef);
				const TraceServices::FStringDefinition* AssetName = DefinitionProvider->Get<TraceServices::FStringDefinition>(AssetNameRef);
				if (PackageName && AssetName)
				{
					Provider.AddMessageTypeInfo(SenderShortId, RecipientShortId, MessageId, PackageName->Display, AssetName->Display);
				}
			}
			break;
		}
		case RouteId_MessageLifecycle:
		{
			FMessageLifecycleEvent Event = FMessageLifecycleEvent::FromEventData(EventData);
			Provider.AddMessageEvent(Context.EventTime.AsSeconds(Event.Cycle), MoveTemp(Event));
			break;
		}
	}

	return true;
}


} // namespace UE::MessagingInsights
