// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices
{
	class IAnalysisSession;
}


namespace UE::MessagingInsights
{


class FUdpMessagingProvider;


class FUdpMessagingAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FUdpMessagingAnalyzer(TraceServices::IAnalysisSession& InSession, FUdpMessagingProvider& InProvider);

	//~ Begin IAnalyzer interface
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;
	//~ End IAnalyzer interface

private:
	enum : uint16
	{
		RouteId_DiscoveredNode,
		RouteId_MessageSummary,
		RouteId_MessageTypeInfo,
		RouteId_MessageLifecycle,
	};

	TraceServices::IAnalysisSession& Session;
	FUdpMessagingProvider& Provider;
};


} // namespace UE::MessagingInsights
