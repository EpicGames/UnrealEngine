// Copyright Epic Games, Inc. All Rights Reserved.

#include "UdpMessagingTraceModule.h"
#include "UdpMessagingTraceAnalyzer.h"
#include "UdpMessagingTraceProvider.h"

namespace UE::MessagingInsights
{

FName FUdpMessagingTraceModule::ModuleName("TraceModule_UdpMessaging");

void FUdpMessagingTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("UdpMessaging");
}

void FUdpMessagingTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedRef<FUdpMessagingProvider> Provider = MakeShared<FUdpMessagingProvider>(InSession);
	InSession.AddProvider(FUdpMessagingProvider::ProviderName, Provider);
	InSession.AddAnalyzer(new FUdpMessagingAnalyzer(InSession, *Provider));
}

} // namespace UE::MessagingInsights
