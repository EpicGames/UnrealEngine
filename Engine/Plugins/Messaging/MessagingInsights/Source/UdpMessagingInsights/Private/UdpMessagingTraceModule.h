// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/ModuleService.h"


namespace UE::MessagingInsights
{

class FUdpMessagingTraceModule : public TraceServices::IModule
{
	static FName ModuleName;

public:
	//~ Begin TraceServices::IModule interface
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	//~ End TraceServices::IModule interface
};

} // namespace UE::MessagingInsights
