// Copyright Epic Games, Inc. All Rights Reserved.

#include "UdpMessagingTraceProvider.h"
#include "Trace/UdpMessagingTrace.h"


namespace UE::MessagingInsights
{

using namespace UE::UdpMessaging::Trace;


const FName FUdpMessagingProvider::ProviderName("UdpMessagingProvider");


FUdpMessagingProvider::FUdpMessagingProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
	, NodeDiscoveredTimeline(Session.GetLinearAllocator())
{
}


void FUdpMessagingProvider::AddNodeDiscoveredEvent(double InEventTimeSeconds, FNodeDiscoveredEvent&& InEvent)
{
	Session.WriteAccessCheck();

	const bool bIsLocalNode = InEvent.IsLocalNode();
	const uint16 ShortId = InEvent.ShortId;
	const uint64 EventIdx = NodeDiscoveredTimeline.EmplaceEvent(InEventTimeSeconds, MoveTemp(InEvent));

	if (ensure(!NodeShortIdToDiscoveredEvent.Contains(ShortId)))
	{
		NodeShortIdToDiscoveredEvent.Add(ShortId, EventIdx);

		const FNodeDiscoveredEvent& Event = NodeDiscoveredTimeline.GetEvent(EventIdx);
		if (bIsLocalNode)
		{
			LocalNodeShortIds.Add(ShortId);
			LocalNodes.Add(ShortId, {
				.SendTimeline{ .MessageTimeline = FMessageTimeline(Session.GetLinearAllocator()) },
				.ReceiveTimeline{ .MessageTimeline = FMessageTimeline(Session.GetLinearAllocator()) },
			});
		}
	}
}


const FNodeDiscoveredEvent* FUdpMessagingProvider::GetNodeByShortId(uint16 InShortId) const
{
	Session.ReadAccessCheck();

	if (const uint64* DiscoveredEvent = NodeShortIdToDiscoveredEvent.Find(InShortId))
	{
		return &NodeDiscoveredTimeline.GetEvent(*DiscoveredEvent);
	}

	return nullptr;
}


namespace Private
{
	bool IsInitialEvent(UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent InEvent)
	{
		switch (InEvent)
		{
			case UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::EnqueueOutbound: // Send
			case UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::FirstSegmentEnqueueInbound: // Receive
				return true;
			default:
				return false;
		}
	}

	bool IsTerminalEvent(UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent InEvent)
	{
		switch (InEvent)
		{
			case UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Complete: // Send
			case UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Delivered: // Receive
			case UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Discarded:
				return true;
			default:
				return false;
		}
	}
}


void FUdpMessagingProvider::AddMessageEvent(double InEventTimeSeconds, FMessageLifecycleEvent&& InEvent)
{
	Session.WriteAccessCheck();

	check(InEvent.Details.IsType<FMessageLifecycleEvent::FInnerEvent>());

	if (FLocalNode* LocalSender = LocalNodes.Find(InEvent.SenderShortId))
	{
		LocalSender->SendTimeline.AddMessageEvent(InEventTimeSeconds, MoveTemp(InEvent));
	}
	else if (FLocalNode* LocalReceiver = LocalNodes.Find(InEvent.RecipientShortId))
	{
		LocalReceiver->ReceiveTimeline.AddMessageEvent(InEventTimeSeconds, MoveTemp(InEvent));
	}
	else
	{
		ensure(false);
	}
}

// See the comment about deferred commit in FLocalNodeTimeline::AddMessageEvent below.
void FUdpMessagingProvider::EnumerateOutstandingMessages(
	uint16 InLocalNodeShortId, EMessageDirection InDirection,
	double InIntervalStart, double InIntervalEnd,
	TraceServices::ITimeline<FMessageLifecycleEvent>::EventRangeCallback InCallback
) const
{
	Session.ReadAccessCheck();

	const FLocalNode* LocalNode = LocalNodes.Find(InLocalNodeShortId);
	if (!LocalNode)
	{
		return;
	}

	const FLocalNodeTimeline& LocalTimeline = (InDirection == EMessageDirection::Send)
		? LocalNode->SendTimeline : LocalNode->ReceiveTimeline;

	TArray<FEventTiming> SortedEventTimings;
	int32 NextUnassignedLane = LocalTimeline.LaneFreeTimes.Num() * 2;
	for (const TPair<FMessageKey, TArray<FEventTiming>>& OutstandingMessage : LocalTimeline.OutstandingMessageEvents)
	{
		SortedEventTimings = OutstandingMessage.Value;
		SortedEventTimings.StableSort([](const FEventTiming& Lhs, const FEventTiming& Rhs) { return Lhs.Value.Cycle < Rhs.Value.Cycle; });

		const double MessageStartTime = SortedEventTimings[0].Key;
		if (MessageStartTime >= InIntervalEnd)
		{
			continue;
		}

		const int32 OuterLaneDepth = NextUnassignedLane;
		NextUnassignedLane += 2;

		FMessageLifecycleEvent TempEvent(SortedEventTimings[0].Value);
		TempEvent.LaneDepth = OuterLaneDepth;
		TempEvent.Details.Set<FMessageLifecycleEvent::FOuterEvent>({
			SortedEventTimings.Last().Value.Details.Get<FMessageLifecycleEvent::FInnerEvent>().EventType });

		InCallback(MessageStartTime, std::numeric_limits<double>::infinity(), 0, TempEvent);

		for (int32 EventIdx = 0; EventIdx < SortedEventTimings.Num(); ++EventIdx)
		{
			const FEventTiming& EventTiming = SortedEventTimings[EventIdx];
			const double StartTime = EventTiming.Key;
			const double EndTime = (EventIdx + 1) < SortedEventTimings.Num()
				? SortedEventTimings[EventIdx + 1].Key
				: std::numeric_limits<double>::infinity();

			TempEvent = EventTiming.Value;
			TempEvent.LaneDepth = OuterLaneDepth + 1;
			InCallback(StartTime, EndTime, 0, TempEvent);
		}
	}
}


void FUdpMessagingProvider::FLocalNodeTimeline::AddMessageEvent(double InEventTimeSeconds, FMessageLifecycleEvent&& InEvent)
{
	const UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent EventType =
		InEvent.Details.Get<FMessageLifecycleEvent::FInnerEvent>().EventType;

	const bool bIsTerminalEvent = Private::IsTerminalEvent(EventType);
	const FMessageKey MessageKey = InEvent.GetMessageKey();
	TArray<FEventTiming>& EventTimings = OutstandingMessageEvents.FindOrAdd(MessageKey);
	EventTimings.Emplace(InEventTimeSeconds, MoveTemp(InEvent));

	// Wait until we've buffered all the events for this message, then commit them in a second pass.
	// Deferring the lane allocation lets us handle events emitted with a retroactive `Cycles` timestamp,
	// or otherwise analyzed out of order, without inadvertently generating overlapping intervals.
	// This means enumerating unterminated messages requires special handling; see EnumerateOutstandingMessages.
	if (bIsTerminalEvent)
	{
		EventTimings.StableSort([](const FEventTiming& Lhs, const FEventTiming& Rhs) { return Lhs.Value.Cycle < Rhs.Value.Cycle; });

		const double StartTime = EventTimings[0].Key;
		const double EndTime = EventTimings.Last().Key;
		const int32 OuterLaneDepth = AllocLaneDepth(StartTime, EndTime);

		// Initialize the outer event as a copy of the first inner event.
		FMessageLifecycleEvent OuterEvent(EventTimings[0].Value);
		OuterEvent.Details.Set<FMessageLifecycleEvent::FOuterEvent>({EventType});
		OuterEvent.LaneDepth = OuterLaneDepth;

		const uint64 OuterEventIdx = MessageTimeline.EmplaceBeginEvent(StartTime, MoveTemp(OuterEvent));
		TOptional<uint64> PrevEventIdx;

		for (FEventTiming& EventTiming : EventTimings)
		{
			if (PrevEventIdx.IsSet())
			{
				MessageTimeline.EndEvent(PrevEventIdx.GetValue(), EventTiming.Key);
			}

			if (!Private::IsTerminalEvent(EventTiming.Value.Details.Get<FMessageLifecycleEvent::FInnerEvent>().EventType))
			{
				EventTiming.Value.LaneDepth = OuterLaneDepth + 1;
				PrevEventIdx = MessageTimeline.EmplaceBeginEvent(EventTiming.Key, MoveTemp(EventTiming.Value));
			}
		}

		MessageTimeline.EndEvent(OuterEventIdx, EndTime);

		OutstandingMessageEvents.Remove(MessageKey); // Note: Invalidates EventTimings
	}
}


int32 FUdpMessagingProvider::FLocalNodeTimeline::AllocLaneDepth(double InStartTime, double InEndTime)
{
	int32 LaneIdx = LaneFreeTimes.IndexOfByPredicate([InStartTime](double Elem) { return Elem < InStartTime; });
	if (LaneIdx == INDEX_NONE)
	{
		LaneIdx = LaneFreeTimes.Add(InEndTime);
	}
	else
	{
		LaneFreeTimes[LaneIdx] = InEndTime;
	}

	return LaneIdx * 2;
}


void FUdpMessagingProvider::AddMessageSummary(uint16 InSenderShortId, uint16 InRecipientShortId, uint32 InMessageId, FMessageSummary&& InMessageSummary)
{
	Session.WriteAccessCheck();
	const FMessageKey MessageKey = MakeMessageKey(InSenderShortId, InRecipientShortId, InMessageId);
	if (ensure(!MessageKeyToSummary.Contains(MessageKey)))
	{
		MessageKeyToSummary.Emplace(MessageKey, MoveTemp(InMessageSummary));
	}
}


const FMessageSummary* FUdpMessagingProvider::GetMessageSummary(uint16 InSenderShortId, uint16 InRecipientShortId, uint32 InMessageId) const
{
	Session.ReadAccessCheck();
	const FMessageKey MessageKey = MakeMessageKey(InSenderShortId, InRecipientShortId, InMessageId);
	return MessageKeyToSummary.Find(MessageKey);
}


void FUdpMessagingProvider::AddMessageTypeInfo(uint16 InSenderShortId, uint16 InRecipientShortId, uint32 InMessageId, const TCHAR* InPackage, const TCHAR* InAsset)
{
	Session.WriteAccessCheck();
	const FMessageKey MessageKey = MakeMessageKey(InSenderShortId, InRecipientShortId, InMessageId);
	if (ensure(!MessageKeyToTypeInfo.Contains(MessageKey)))
	{
		MessageKeyToTypeInfo.Add(MessageKey, { InPackage, InAsset });
	}
}


TPair<const TCHAR*, const TCHAR*> FUdpMessagingProvider::GetMessageTypeInfo(uint16 SenderShortId, uint16 RecipientShortId, uint32 MessageId) const
{
	Session.ReadAccessCheck();
	const FMessageKey MessageKey = MakeMessageKey(SenderShortId, RecipientShortId, MessageId);
	if (const TPair<const TCHAR*, const TCHAR*>* TypeInfo = MessageKeyToTypeInfo.Find(MessageKey))
	{
		return *TypeInfo;
	}

	return { nullptr, nullptr };
}


FNodeDiscoveredEvent FNodeDiscoveredEvent::FromEventData(const UE::Trace::IAnalyzer::FEventData& EventData)
{
	return {
		.Cycle = EventData.GetValue<uint64>("Cycle"),
		.ShortId = EventData.GetValue<uint16>("ShortId"),
		.DiscoveredByShortId = EventData.GetValue<uint16>("DiscoveredByShortId"),
		.Id = FGuid{
			EventData.GetValue<uint32>("IdA"),
			EventData.GetValue<uint32>("IdB"),
			EventData.GetValue<uint32>("IdC"),
			EventData.GetValue<uint32>("IdD"),
		},
		.Endpoint = {
			FIPv4Address(
				EventData.GetValue<uint8>("IpAddrA"),
				EventData.GetValue<uint8>("IpAddrB"),
				EventData.GetValue<uint8>("IpAddrC"),
				EventData.GetValue<uint8>("IpAddrD")
			),
			EventData.GetValue<uint16>("Port")
		},
		.ProtocolVersion = EventData.GetValue<uint8>("ProtocolVersion"),
	};
}

FMessageSummary FMessageSummary::FromEventData(const UE::Trace::IAnalyzer::FEventData& EventData)
{
	return {
		.SenderShortId = EventData.GetValue<uint16>("SenderShortId"),
		.RecipientShortId = EventData.GetValue<uint16>("RecipientShortId"),
		.MessageId = EventData.GetValue<uint32>("MessageId"),
		.Size = EventData.GetValue<uint64>("Size"),
		.NumSegments = EventData.GetValue<uint32>("NumSegments"),
	};
}


FMessageLifecycleEvent FMessageLifecycleEvent::FromEventData(const UE::Trace::IAnalyzer::FEventData& EventData)
{
	return {
		.Cycle = EventData.GetValue<uint64>("Cycle"),
		.SenderShortId = EventData.GetValue<uint16>("SenderShortId"),
		.RecipientShortId = EventData.GetValue<uint16>("RecipientShortId"),
		.MessageId = EventData.GetValue<uint32>("MessageId"),
		.Details = TVariant<FOuterEvent, FInnerEvent>(TInPlaceType<FInnerEvent>(),
			static_cast<EUdpMessageLifecycleEvent>(
				EventData.GetValue<std::underlying_type_t<EUdpMessageLifecycleEvent>>("LifecycleEvent"))
		),
	};
}


} // namespace UE::MessagingInsights
