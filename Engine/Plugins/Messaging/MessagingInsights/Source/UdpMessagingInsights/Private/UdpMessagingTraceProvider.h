// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Model/IntervalTimeline.h"
#include "Model/PointTimeline.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace UE::UdpMessaging::Trace
{
	enum class EUdpMessageLifecycleEvent : uint8;
}


namespace UE::MessagingInsights
{
using FMessageKey = uint64;

inline FMessageKey MakeMessageKey(uint16 SenderShortId, uint16 RecipientShortId, uint32 MessageId)
{
	return
		uint64(SenderShortId)    << 48 |
		uint64(RecipientShortId) << 32 |
		MessageId;
}

struct FNodeDiscoveredEvent
{
	uint64 Cycle;
	uint16 ShortId;
	uint16 DiscoveredByShortId;
	FGuid Id;
	FIPv4Endpoint Endpoint;
	uint8 ProtocolVersion;

	static FNodeDiscoveredEvent FromEventData(const UE::Trace::IAnalyzer::FEventData& EventData);

	bool IsLocalNode() const
	{
		return DiscoveredByShortId == 0;
	}
};

struct FMessageSummary
{
	uint16 SenderShortId;
	uint16 RecipientShortId;
	uint32 MessageId;
	uint64 Size;
	uint32 NumSegments;

	static FMessageSummary FromEventData(const UE::Trace::IAnalyzer::FEventData& EventData);
};

struct FMessageLifecycleEvent
{
	uint64 Cycle;
	uint16 SenderShortId;
	uint16 RecipientShortId;
	uint32 MessageId;

	struct FOuterEvent
	{
		UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent TerminalEventType;
	};

	struct FInnerEvent
	{
		UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent EventType;
	};

	TVariant<FOuterEvent, FInnerEvent> Details;

	// Computed/derived values
	int32 LaneDepth = 0;

	static FMessageLifecycleEvent FromEventData(const UE::Trace::IAnalyzer::FEventData& EventData);

	FMessageKey GetMessageKey() const
	{
		return MakeMessageKey(SenderShortId, RecipientShortId, MessageId);
	}
};


enum class EMessageDirection : uint8
{
	Send,
	Receive,
};


class FUdpMessagingProvider : public TraceServices::IProvider
{
	using FEventTiming = TPair<double, FMessageLifecycleEvent>;

public:
	static const FName ProviderName;

	FUdpMessagingProvider(TraceServices::IAnalysisSession& InSession);

	using FNodeDiscoveredTimeline = TraceServices::TPointTimeline<FNodeDiscoveredEvent>;
	const FNodeDiscoveredTimeline& GetNodeDiscoveredTimeline() const
	{
		Session.ReadAccessCheck();
		return NodeDiscoveredTimeline;
	}
	void AddNodeDiscoveredEvent(double InEventTimeSeconds, FNodeDiscoveredEvent&& InEvent);

	using FMessageTimeline = TraceServices::TIntervalTimeline<FMessageLifecycleEvent>;
	const FMessageTimeline* GetMessageTimeline(uint16 InLocalNodeShortId, EMessageDirection InDirection) const
	{
		Session.ReadAccessCheck();
		if (const FLocalNode* LocalNode = LocalNodes.Find(InLocalNodeShortId); ensure(LocalNode))
		{
			switch (InDirection)
			{
				case EMessageDirection::Send: return &LocalNode->SendTimeline.MessageTimeline;
				case EMessageDirection::Receive: return &LocalNode->ReceiveTimeline.MessageTimeline;
				default: checkNoEntry();
			}
		}

		return nullptr;
	}

	void EnumerateOutstandingMessages(
		uint16 InLocalNodeShortId, EMessageDirection InDirection,
		double InIntervalStart, double InIntervalEnd,
		TraceServices::ITimeline<FMessageLifecycleEvent>::EventRangeCallback InCallback
	) const;

	uint32 GetNumLocalNodes() const
	{
		Session.ReadAccessCheck();
		return LocalNodeShortIds.Num();
	}

	uint16 GetLocalNodeShortId(uint32 InLocalNodeIdx) const
	{
		Session.ReadAccessCheck();
		check(InLocalNodeIdx < (uint32)LocalNodeShortIds.Num());
		return LocalNodeShortIds[InLocalNodeIdx];
	}

	const FNodeDiscoveredEvent* GetNodeByShortId(uint16 InShortId) const;

	void AddMessageEvent(double InEventTimeSeconds, FMessageLifecycleEvent&& InEvent);

	void AddMessageSummary(uint16 InSenderShortId, uint16 InRecipientShortId, uint32 InMessageId, FMessageSummary&& InMessageSummary);
	const FMessageSummary* GetMessageSummary(uint16 InSenderShortId, uint16 InRecipientShortId, uint32 InMessageId) const;

	void AddMessageTypeInfo(uint16 InSenderShortId, uint16 InRecipientShortId, uint32 InMessageId, const TCHAR* InPackage, const TCHAR* InAsset);
	TPair<const TCHAR*, const TCHAR*> GetMessageTypeInfo(uint16 InSenderShortId, uint16 InRecipientShortId, uint32 InMessageId) const;

private:
	TraceServices::IAnalysisSession& Session;

	FNodeDiscoveredTimeline NodeDiscoveredTimeline;
	TMap<uint16, uint64> NodeShortIdToDiscoveredEvent;

	/** Contains the short IDs just for local nodes (typically just a few). */
	TArray<uint16> LocalNodeShortIds;

	struct FLocalNodeTimeline
	{
		FMessageTimeline MessageTimeline;
		TArray<double> LaneFreeTimes;

		TMap<FMessageKey, TArray<FEventTiming>> OutstandingMessageEvents;

		void AddMessageEvent(double InEventTimeSeconds, FMessageLifecycleEvent&& InEvent);
		int32 AllocLaneDepth(double InStartTime, double InEndTime);
	};

	struct FLocalNode
	{
		FLocalNodeTimeline SendTimeline;
		FLocalNodeTimeline ReceiveTimeline;
	};

	TMap<uint16, FLocalNode> LocalNodes;

	// Pointers are to strings owned by TraceServices::FStringDefinition, guaranteed valid for the analysis session.
	TMap<FMessageKey, TPair<const TCHAR*, const TCHAR*>> MessageKeyToTypeInfo;

	TMap<FMessageKey, FMessageSummary> MessageKeyToSummary;
};


} // namespace UE::MessagingInsights
