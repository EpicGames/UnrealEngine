// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UdpMessagingInsights : ModuleRules
	{
		public UdpMessagingInsights(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"Messaging",
				"Networking",
				"Slate",
				"SlateCore",
				"TraceAnalysis",
				"TraceLog",
				"TraceInsights",
				"TraceInsightsCore",
				"TraceServices",
				"UdpMessaging",
			});

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.AddRange(new string[]
				{
					"Engine",
				});
			}
		}
	}
}
